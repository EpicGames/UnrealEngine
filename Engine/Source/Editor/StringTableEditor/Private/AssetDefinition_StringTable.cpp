// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_StringTable.h"
#include "StringTableEditorModule.h"
#include "Modules/ModuleManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetDefinition_StringTable)

#define LOCTEXT_NAMESPACE "AssetTypeActions"

EAssetCommandResult UAssetDefinition_StringTable::OpenAssets(const FAssetOpenArgs& OpenArgs) const
{
	FStringTableEditorModule& StringTableEditorModule = FModuleManager::LoadModuleChecked<FStringTableEditorModule>("StringTableEditor");

	for (UStringTable* StringTable : OpenArgs.LoadObjects<UStringTable>())
	{
		StringTableEditorModule.CreateStringTableEditor(OpenArgs.GetToolkitMode(), OpenArgs.ToolkitHost, StringTable);
	}

	return EAssetCommandResult::Handled;
}

#undef LOCTEXT_NAMESPACE
