// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Internationalization/StringTable.h"
#include "AssetDefinitionDefault.h"

#include "AssetDefinition_StringTable.generated.h"

UCLASS()
class UAssetDefinition_StringTable : public UAssetDefinitionDefault
{
	GENERATED_BODY()

public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_StringTable", "String Table"); }
	virtual FLinearColor GetAssetColor() const override { return FLinearColor(FColor(255, 196, 128)); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UStringTable::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const TArray<FAssetCategoryPath> Categories = { FAssetCategoryPath(EAssetCategoryPaths::Data, NSLOCTEXT("AssetDefinition", "StringTable", "Data Table"), ECategoryMenuType::Section) };
		return Categories;
	}
	virtual FAssetSupportResponse CanLocalize(const FAssetData& InAsset) const override
	{
		return FAssetSupportResponse::NotSupported();
	}
	virtual EAssetCommandResult OpenAssets(const FAssetOpenArgs& OpenArgs) const override;
	// UAssetDefinition End
};
