// Copyright Epic Games, Inc. All Rights Reserved.

#include "StringTableEditor.h"
#include "Misc/FileHelper.h"
#include "Modules/ModuleManager.h"
#include "Styling/AppStyle.h"
#include "EditorDirectories.h"
#include "DesktopPlatformModule.h"
#include "StringTableEditorModule.h"
#include "Internationalization/StringTable.h"
#include "Internationalization/StringTableCore.h"
#include "Internationalization/TextNamespaceUtil.h"
#include "Editor.h"
#include "ScopedTransaction.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/Commands/GenericCommands.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"

#include "Widgets/Input/SButton.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SHeaderRow.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
 
#define LOCTEXT_NAMESPACE "StringTableEditor"

const FName FStringTableEditor::StringTableTabId("StringTableEditor_StringTable");
const FName FStringTableEditor::StringTableDummyColumnId("Dummy");
const FName FStringTableEditor::StringTableKeyColumnId("Key");
const FName FStringTableEditor::StringTableSourceStringColumnId("SourceString");
const FName FStringTableEditor::StringTableDevNotesColumnId("DevNotes");
const FName FStringTableEditor::StringTableDeleteColumnId("Delete");

class SStringTableEntryRow : public SMultiColumnTableRow<TSharedRef<FStringTableEditor::FCachedStringTableEntry>>
{
public:
	void Construct(const FTableRowArgs& InArgs, const TSharedRef<STableViewBase>& OwnerTableView, FStringTableEditor& InOwnerEditor, TSharedRef<FStringTableEditor::FCachedStringTableEntry> InCachedStringTableEntry)
	{
		OwnerEditor = &InOwnerEditor;
		CachedStringTableEntry = InCachedStringTableEntry;

		FSuperRowType::Construct(InArgs, OwnerTableView);
	}

	TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override
	{
		TSharedPtr<SWidget> Return;

		if (ColumnName == FStringTableEditor::StringTableDummyColumnId)
		{
			Return = SNew(SBorder)
				.BorderImage(&FCoreStyle::Get().GetWidgetStyle<FHeaderRowStyle>("TableView.Header").BackgroundBrush);
		}
		else if (ColumnName == FStringTableEditor::StringTableKeyColumnId)
		{
			Return = SNew(SBox)
				.Padding(4)
				[
					SNew(SInlineEditableTextBlock)
					.IsReadOnly(true)
					.Text(FText::FromString(CachedStringTableEntry->Key))
				];
		}
		else if (ColumnName == FStringTableEditor::StringTableSourceStringColumnId)
		{
			Return = SNew(SBorder)
				.BorderImage(&FCoreStyle::Get().GetWidgetStyle<FEditableTextBoxStyle>("NormalEditableTextBox").BackgroundImageReadOnly)
				[
					SNew(SMultiLineEditableTextBox)
					.Text(FText::FromString(CachedStringTableEntry->SourceString))
					.ModiferKeyForNewLine(EModifierKey::Shift)
					.OnTextCommitted(this, &SStringTableEntryRow::OnSourceStringCommitted)
				];
		}
		else if (ColumnName == FStringTableEditor::StringTableDevNotesColumnId)
		{
			Return = SNew(SBorder)
				.BorderImage(&FCoreStyle::Get().GetWidgetStyle<FEditableTextBoxStyle>("NormalEditableTextBox").BackgroundImageReadOnly)
				[
					SNew(SMultiLineEditableTextBox)
					.Text(FText::FromString(CachedStringTableEntry->DevNotes))
					.ModiferKeyForNewLine(EModifierKey::Shift)
					.OnTextCommitted(this, &SStringTableEntryRow::OnDevNotesCommitted)
				];
		}
		else if (ColumnName == FStringTableEditor::StringTableDeleteColumnId)
		{
			Return = SNew(SButton)
				.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
				.ForegroundColor(FSlateColor::UseForeground())
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.OnClicked(this, &SStringTableEntryRow::OnDeleteEntryClicked)
				.ToolTipText(LOCTEXT("DeleteEntryTooltip", "Delete this entry from the string table"))
				[
					SNew(SImage)
					.Image(FAppStyle::Get().GetBrush("Cross"))
				];
		}

		return Return.IsValid() ? Return.ToSharedRef() : SNullWidget::NullWidget;
	}

private:
	void OnSourceStringCommitted(const FText& InText, ETextCommit::Type)
	{
		const FString NewSourceString = InText.ToString();
		if (!NewSourceString.IsEmpty() && !CachedStringTableEntry->SourceString.Equals(NewSourceString, ESearchCase::CaseSensitive))
		{
			CachedStringTableEntry->SourceString = NewSourceString;
			OwnerEditor->SetEntry(CachedStringTableEntry->Key, CachedStringTableEntry->SourceString, CachedStringTableEntry->DevNotes);
		}
	}

	void OnDevNotesCommitted(const FText& InText, ETextCommit::Type)
	{
		const FString NewDevNotes = InText.ToString();
		if (!CachedStringTableEntry->DevNotes.Equals(NewDevNotes, ESearchCase::CaseSensitive))
		{
			CachedStringTableEntry->DevNotes = NewDevNotes;
			OwnerEditor->SetEntry(CachedStringTableEntry->Key, CachedStringTableEntry->SourceString, CachedStringTableEntry->DevNotes);
		}
	}

	FReply OnDeleteEntryClicked()
	{
		OwnerEditor->DeleteEntry(CachedStringTableEntry->Key);
		return FReply::Handled();
	}

	FStringTableEditor* OwnerEditor;
	TSharedPtr<FStringTableEditor::FCachedStringTableEntry> CachedStringTableEntry;
};

void FStringTableEditor::RegisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager)
{
	WorkspaceMenuCategory = InTabManager->AddLocalWorkspaceMenuCategory(LOCTEXT("WorkspaceMenu_StringTableEditor", "String Table Editor"));

	FAssetEditorToolkit::RegisterTabSpawners(InTabManager);

	InTabManager->RegisterTabSpawner(StringTableTabId, FOnSpawnTab::CreateSP(this, &FStringTableEditor::SpawnTab_StringTable) )
		.SetDisplayName(LOCTEXT("StringTableTab", "String Table"))
		.SetGroup(WorkspaceMenuCategory.ToSharedRef());
}

void FStringTableEditor::UnregisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager)
{
	FAssetEditorToolkit::UnregisterTabSpawners(InTabManager);

	InTabManager->UnregisterTabSpawner(StringTableTabId);
}

FStringTableEditor::~FStringTableEditor()
{
	GEditor->UnregisterForUndo(this);
}

void FStringTableEditor::PostUndo(bool bSuccess)
{
	HandleUndoRedo();
}

void FStringTableEditor::PostRedo(bool bSuccess)
{
	HandleUndoRedo();
}

void FStringTableEditor::HandleUndoRedo()
{
	const UStringTable* StringTable = GetStringTable();
	if (StringTable)
	{
		HandlePostChange();
	}
}

const UStringTable* FStringTableEditor::GetStringTable() const
{
	return Cast<const UStringTable>(GetEditingObject());
}

void FStringTableEditor::RefreshStringTableEditor(const FString& NewSelection)
{
	HandlePostChange(NewSelection);
}

void FStringTableEditor::HandlePostChange(const FString& NewSelection)
{
	// We need to cache and restore the selection here as RefreshCachedStringTable will re-create the list view items
	FString CachedSelection = NewSelection;
	if (CachedSelection.IsEmpty())
	{
		TArray<TSharedPtr<FCachedStringTableEntry>> SelectedEntries;
		StringTableEntriesListView->GetSelectedItems(SelectedEntries);
		if (SelectedEntries.Num() == 1)
		{
			CachedSelection = SelectedEntries[0]->Key;
		}
	}

	RefreshCachedStringTable(CachedSelection);
}

void FStringTableEditor::InitStringTableEditor(const EToolkitMode::Type Mode, const TSharedPtr<class IToolkitHost>& InitToolkitHost, UStringTable* StringTable)
{
	TSharedRef<FTabManager::FLayout> StandaloneDefaultLayout = FTabManager::NewLayout("Standalone_StringTableEditor_Layout_v2")
	->AddArea
	(
		FTabManager::NewPrimaryArea()->SetOrientation(Orient_Vertical)
		->Split
		(
			FTabManager::NewStack()
			->SetHideTabWell(true)
			->AddTab(StringTableTabId, ETabState::OpenedTab)
		)
	);

	const bool bCreateDefaultStandaloneMenu = true;
	const bool bCreateDefaultToolbar = true;
	FAssetEditorToolkit::InitAssetEditor(Mode, InitToolkitHost, FStringTableEditorModule::StringTableEditorAppIdentifier, StandaloneDefaultLayout, bCreateDefaultStandaloneMenu, bCreateDefaultToolbar, StringTable);
	
	FStringTableEditorModule& StringTableEditorModule = FModuleManager::LoadModuleChecked<FStringTableEditorModule>("StringTableEditor");
	AddMenuExtender(StringTableEditorModule.GetMenuExtensibilityManager()->GetAllExtenders(GetToolkitCommands(), GetEditingObjects()));
	AddToolbarExtender(StringTableEditorModule.GetToolBarExtensibilityManager()->GetAllExtenders(GetToolkitCommands(), GetEditingObjects()));

	RegenerateMenusAndToolbars();

	// Support undo/redo
	GEditor->RegisterForUndo(this);
}

FName FStringTableEditor::GetToolkitFName() const
{
	return FName("StringTableEditor");
}

FText FStringTableEditor::GetBaseToolkitName() const
{
	return LOCTEXT("AppLabel", "String Table Editor");
}

FString FStringTableEditor::GetWorldCentricTabPrefix() const
{
	return LOCTEXT("WorldCentricTabPrefix", "StringTable ").ToString();
}

FLinearColor FStringTableEditor::GetWorldCentricTabColorScale() const
{
	return FLinearColor(0.0f, 0.0f, 0.2f, 0.5f);
}

TSharedRef<SDockTab> FStringTableEditor::SpawnTab_StringTable(const FSpawnTabArgs& Args)
{
	check(Args.GetTabId().TabType == StringTableTabId);

	const FGenericCommands& GenericCommands = FGenericCommands::Get();
	CommandList = MakeShareable(new FUICommandList);
	CommandList->MapAction(
		GenericCommands.Delete,
		FExecuteAction::CreateSP(this, &FStringTableEditor::ExecuteDelete),
		FCanExecuteAction::CreateSP(this, &FStringTableEditor::CanExecuteDelete));

	UStringTable* StringTable = Cast<UStringTable>(GetEditingObject());

	// Support undo/redo
	if (StringTable)
	{
		StringTable->SetFlags(RF_Transactional);
	}

	EntryTextFilter = MakeShareable(new FEntryTextFilter(FEntryTextFilter::FItemToStringArray::CreateLambda([](const TSharedPtr<FCachedStringTableEntry>& InItem, OUT TArray< FString >& StringArray) {
		StringArray.Add(InItem->Key);
		StringArray.Add(InItem->SourceString);
		StringArray.Add(InItem->DevNotes);
	})));

	TSharedRef<SDockTab> StringTableTab = SNew(SDockTab)
		.Label(LOCTEXT("StringTableTitle", "String Table"))
		.TabColorScale(GetTabColorScale())
		[
			SNew(SVerticalBox)

			+SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SBorder)
				.Padding(2)
				.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
				[
					SNew(SHorizontalBox)

					+SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(2)
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("NamespaceLabel", "Namespace:"))
					]

					+SHorizontalBox::Slot()
					.Padding(2)
					.VAlign(VAlign_Center)
					[
						SAssignNew(NamespaceEditableTextBox, SEditableTextBox)
						.Text(this, &FStringTableEditor::GetNamespace)
						.OnTextChanged(this, &FStringTableEditor::OnNamespaceChanged)
						.OnTextCommitted(this, &FStringTableEditor::OnNamespaceCommitted)
					]

					+SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(2)
					.VAlign(VAlign_Center)
					[
						SNew(SButton)
						.Text(LOCTEXT("ImportFromCSVLabel", "Import from CSV"))
						.OnClicked(this, &FStringTableEditor::OnImportFromCSVClicked)
						.ButtonStyle(FAppStyle::Get(), "FlatButton.Default")
						.ForegroundColor(FLinearColor::White)
						.ContentPadding(FMargin(6, 2))
					]

					+SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(2)
					.VAlign(VAlign_Center)
					[
						SNew(SButton)
						.Text(LOCTEXT("ExportToCSVLabel", "Export to CSV"))
						.OnClicked(this, &FStringTableEditor::OnExportToCSVClicked)
						.ButtonStyle(FAppStyle::Get(), "FlatButton.Default")
						.ForegroundColor(FLinearColor::White)
						.ContentPadding(FMargin(6, 2))
					]
				]
			]

			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(0, 4, 0, 4)
			[
				SNew(SBorder)
				.Padding(2)
				.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
				[
					SAssignNew(SearchBox, SSearchBox)
					.OnTextChanged(this, &FStringTableEditor::OnFilterTextChanged)
				]
			]

			+SVerticalBox::Slot()
			[
				SNew(SBorder)
				.Padding(2)
				.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
				[
					SAssignNew(StringTableEntriesListView, SListView<TSharedPtr<FCachedStringTableEntry>>)
					.ListItemsSource(&CachedStringTableEntries)
					.OnGenerateRow(this, &FStringTableEditor::OnGenerateStringTableEntryRow)
					.SelectionMode(ESelectionMode::Multi)
					.OnContextMenuOpening(this, &FStringTableEditor::OnConstructContextMenu)
					.HeaderRow
					(
						SNew(SHeaderRow)

						+SHeaderRow::Column(StringTableDummyColumnId)
						.DefaultLabel(FText::GetEmpty())
						.FixedWidth(20)

						+SHeaderRow::Column(StringTableKeyColumnId)
						.DefaultLabel(LOCTEXT("KeyColumnLabel", "Key"))
						.FillWidth(0.2f)

						+SHeaderRow::Column(StringTableSourceStringColumnId)
						.DefaultLabel(LOCTEXT("SourceStringColumnLabel", "Source String"))
						.FillWidth(1.0f)

						+SHeaderRow::Column(StringTableDevNotesColumnId)
						.DefaultLabel(LOCTEXT("DevNotesColumnLabel", "Developer Notes"))
						.ToolTipText(LOCTEXT("DenNotesColumnLabelToolTip", "Notes to give translators additional context that will aid them in translating the source string (details on format parameters, homograph disambiguation, tone, etc)"))
						.FillWidth(0.5f)

						+SHeaderRow::Column(StringTableDeleteColumnId)
						.DefaultLabel(FText::GetEmpty())
						.FixedWidth(28)
					)
				]
			]

			+SVerticalBox::Slot()
			.Padding(0, 4, 0, 0)
			.AutoHeight()
			[
				SNew(SBorder)
				.Padding(2)
				.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
				[
					SNew(SHorizontalBox)

					+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(2)
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("KeyLabel", "Key:"))
					]

					+SHorizontalBox::Slot()
					.FillWidth(0.2f)
					.Padding(2)
					.VAlign(VAlign_Center)
					[
						SAssignNew(KeyEditableTextBox, SEditableTextBox)
						.OnTextChanged(this, &FStringTableEditor::OnKeyChanged)
					]

					+SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(2)
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("SourceStringLabel", "Source String:"))
					]

					+SHorizontalBox::Slot()
					.Padding(2)
					.VAlign(VAlign_Center)
					[
						SAssignNew(SourceStringEditableTextBox, SMultiLineEditableTextBox)
						.ModiferKeyForNewLine(EModifierKey::Shift)
						.OnTextCommitted(this, &FStringTableEditor::OnSourceStringCommitted)
					]

					+SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(2)
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("DevNotesLabel", "Dev Notes:"))
					]

					+SHorizontalBox::Slot()
					.Padding(2)
					.VAlign(VAlign_Center)
					[
						SAssignNew(DevNotesEditableTextBox, SMultiLineEditableTextBox)
						.ModiferKeyForNewLine(EModifierKey::Shift)
						.OnTextCommitted(this, &FStringTableEditor::OnSourceStringCommitted)
					]

					+SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(2)
					.VAlign(VAlign_Center)
					[
						SNew(SButton)
						.Text(LOCTEXT("AddLabel", "Add"))
						.OnClicked(this, &FStringTableEditor::OnAddClicked)
						.ButtonStyle(FAppStyle::Get(), "FlatButton.Success")
						.ForegroundColor(FLinearColor::White)
						.ContentPadding(FMargin(6, 2))
					]
				]
			]
		];

	RefreshCachedStringTable();

	return StringTableTab;
}

TSharedPtr<SWidget> FStringTableEditor::OnConstructContextMenu()
{
	const bool bShouldCloseWindowAfterMenuSelection = true;
	FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, CommandList);

	MenuBuilder.BeginSection("EditOperations", LOCTEXT("StringTableEditOperationsLabel", "Edit"));
	{
		MenuBuilder.AddMenuEntry(FGenericCommands::Get().Delete);
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

TSharedRef<ITableRow> FStringTableEditor::OnGenerateStringTableEntryRow(TSharedPtr<FCachedStringTableEntry> CachedStringTableEntry, const TSharedRef<STableViewBase>& Table)
{
	return SNew(SStringTableEntryRow, Table, *this, CachedStringTableEntry.ToSharedRef());
}

void FStringTableEditor::RefreshCachedStringTable(const FString& InCachedSelection)
{
	CachedStringTableEntries.Reset();

	TSharedPtr<FCachedStringTableEntry> SelectedStringTableEntry;

	UStringTable* StringTable = Cast<UStringTable>(GetEditingObject());
	if (StringTable)
	{
		StringTable->GetStringTable()->EnumerateSourceStrings([&](const FString& InKey, const FString& InSourceString, const FString& InDevNotes)
		{
			TSharedRef<FCachedStringTableEntry> NewStringTableEntry = MakeShared<FCachedStringTableEntry>(InKey, InSourceString, InDevNotes);

			if (EntryTextFilter->PassesFilter(NewStringTableEntry))
			{
				CachedStringTableEntries.Add(NewStringTableEntry);
			}

			if (InCachedSelection.Equals(InKey, ESearchCase::CaseSensitive))
			{
				SelectedStringTableEntry = NewStringTableEntry;
			}

			return true; // continue enumeration
		});
	}

	CachedStringTableEntries.Sort([](const TSharedPtr<FCachedStringTableEntry>& InEntryOne, const TSharedPtr<FCachedStringTableEntry>& InEntryTwo)
	{
		return InEntryOne->Key < InEntryTwo->Key;
	});

	StringTableEntriesListView->RequestListRefresh();

	if (!SelectedStringTableEntry.IsValid() && CachedStringTableEntries.Num() > 0)
	{
		SelectedStringTableEntry = CachedStringTableEntries[0];
	}

	if (SelectedStringTableEntry.IsValid())
	{
		StringTableEntriesListView->SetSelection(SelectedStringTableEntry);
		StringTableEntriesListView->RequestScrollIntoView(SelectedStringTableEntry);
	}
	else
	{
		StringTableEntriesListView->ClearSelection();
	}
}

FText FStringTableEditor::GetNamespace() const
{
	UStringTable* StringTable = Cast<UStringTable>(GetEditingObject());
	if (StringTable)
	{
		return FText::FromString(StringTable->GetStringTable()->GetNamespace());
	}
	return FText::GetEmpty();
}

void FStringTableEditor::OnNamespaceChanged(const FText& InText)
{
	FText ErrorText;
	const FText ErrorCtx = LOCTEXT("TextNamespaceErrorCtx", "Namespace");
	IsValidIdentity(InText, &ErrorText, &ErrorCtx);

	NamespaceEditableTextBox->SetError(ErrorText);
}

void FStringTableEditor::OnNamespaceCommitted(const FText& InText, ETextCommit::Type)
{
	if (IsValidIdentity(InText))
	{
		UStringTable* StringTable = Cast<UStringTable>(GetEditingObject());
		if (StringTable)
		{
			const FString NewNamespace = InText.ToString();
			if (!StringTable->GetStringTable()->GetNamespace().Equals(NewNamespace, ESearchCase::CaseSensitive))
			{
				const FScopedTransaction Transaction(LOCTEXT("SetNamespace", "Set Namespace"));
				StringTable->Modify();
				StringTable->GetMutableStringTable()->SetNamespace(NewNamespace);
				HandlePostChange();
			}
		}
	}
}

void FStringTableEditor::SetEntry(const FString& InKey, const FString& InSourceString, const FString& InDevNotes)
{
	UStringTable* StringTable = Cast<UStringTable>(GetEditingObject());
	if (StringTable)
	{
		const FScopedTransaction Transaction(LOCTEXT("SetStringTableEntry", "Set String Table Entry"));
		StringTable->Modify();
		StringTable->GetMutableStringTable()->SetSourceString(InKey, InSourceString, InDevNotes);
		HandlePostChange(InKey);
	}
}

void FStringTableEditor::DeleteEntry(const FString& InKey)
{
	UStringTable* StringTable = Cast<UStringTable>(GetEditingObject());
	if (StringTable)
	{
		const FScopedTransaction Transaction(LOCTEXT("DeleteStringTableEntry", "Delete String Table Entry"));
		StringTable->Modify();
		StringTable->GetMutableStringTable()->RemoveSourceString(InKey);
		HandlePostChange();
	}
}

bool FStringTableEditor::IsValidIdentity(const FText& InIdentity, FText* OutReason, const FText* InErrorCtx) const
{
	const FString InvalidIdentityChars = FString::Printf(TEXT("%s%c%c"), INVALID_NAME_CHARACTERS, TextNamespaceUtil::PackageNamespaceStartMarker, TextNamespaceUtil::PackageNamespaceEndMarker);
	return FName::IsValidXName(InIdentity.ToString(), InvalidIdentityChars, OutReason, InErrorCtx);
}

void FStringTableEditor::OnKeyChanged(const FText& InText)
{
	FText ErrorText;
	const FText ErrorCtx = LOCTEXT("TextKeyErrorCtx", "Key");
	const bool bIsValidName = IsValidIdentity(InText, &ErrorText, &ErrorCtx);

	if (InText.IsEmptyOrWhitespace())
	{
		ErrorText = LOCTEXT("Error_EmptyKey", "Key cannot be empty.");
	}
	else if (bIsValidName)
	{
		UStringTable* StringTable = Cast<UStringTable>(GetEditingObject());
		if (StringTable)
		{
			const FString NewKey = KeyEditableTextBox->GetText().ToString();
			if (StringTable->GetStringTable()->FindEntry(NewKey).IsValid())
			{
				ErrorText = LOCTEXT("Warning_DuplicateKey", "This key is already being used by this string table. Adding this entry will replace the existing entry.");
			}
		}
	}

	KeyEditableTextBox->SetError(ErrorText);
}

void FStringTableEditor::OnSourceStringCommitted(const FText& InText, ETextCommit::Type InCommitType)
{
	if (InCommitType == ETextCommit::OnEnter)
	{
		OnAddClicked();
	}
}

void FStringTableEditor::OnFilterTextChanged(const FText& InNewText)
{
	EntryTextFilter->SetRawFilterText(InNewText);
	SearchBox->SetError(EntryTextFilter->GetFilterErrorText());

	RefreshCachedStringTable();

	StringTableEntriesListView->RebuildList();
}

bool FStringTableEditor::CanExecuteDelete() const
{
	return StringTableEntriesListView->GetNumItemsSelected() > 0;
}

void FStringTableEditor::ExecuteDelete()
{
	const TArray<TSharedPtr<FCachedStringTableEntry>>& Items = StringTableEntriesListView->GetSelectedItems();
	for (const auto& Item : Items)
	{
		DeleteEntry(Item->Key);
	}
}

FReply FStringTableEditor::OnAddClicked()
{
	const FText NewKey = KeyEditableTextBox->GetText();
	const FText NewSourceString = SourceStringEditableTextBox->GetText();

	if (!NewKey.IsEmptyOrWhitespace() && IsValidIdentity(NewKey) && !NewSourceString.IsEmpty())
	{
		SetEntry(NewKey.ToString(), NewSourceString.ToString(), DevNotesEditableTextBox->GetText().ToString());

		KeyEditableTextBox->SetText(FText::GetEmpty());
		SourceStringEditableTextBox->SetText(FText::GetEmpty());
		DevNotesEditableTextBox->SetText(FText::GetEmpty());

		KeyEditableTextBox->SetError(FText::GetEmpty());
	}

	return FReply::Handled();
}

FReply FStringTableEditor::OnImportFromCSVClicked()
{
	IDesktopPlatform* const DesktopPlatform = FDesktopPlatformModule::Get();

	if (DesktopPlatform)
	{
		const FString DefaultPath = FEditorDirectories::Get().GetLastDirectory(ELastDirectory::GENERIC_IMPORT);

		TSharedPtr<SWindow> ParentWindow = FSlateApplication::Get().FindWidgetWindow(StringTableEntriesListView.ToSharedRef());
		const void* const ParentWindowHandle = (ParentWindow.IsValid() && ParentWindow->GetNativeWindow().IsValid())
			? ParentWindow->GetNativeWindow()->GetOSWindowHandle()
			: nullptr;

		TArray<FString> OutFiles;
		if (DesktopPlatform->OpenFileDialog(
			ParentWindowHandle,
			LOCTEXT("ImportStringTableTitle", "Choose a string table CSV file...").ToString(),
			DefaultPath,
			TEXT(""),
			TEXT("String Table CSV (*.csv)|*.csv"),
			EFileDialogFlags::None,
			OutFiles
			))
		{
			UStringTable* StringTable = Cast<UStringTable>(GetEditingObject());
			if (StringTable)
			{
				const FScopedTransaction Transaction(LOCTEXT("ImportStringTableEntries", "Import String Table Entries"));
				StringTable->Modify();
				StringTable->GetMutableStringTable()->ImportStringsFromCSVFile(OutFiles[0]);
				HandlePostChange();
			}
		}
	}

	return FReply::Handled();
}

FReply FStringTableEditor::OnExportToCSVClicked()
{
	IDesktopPlatform* const DesktopPlatform = FDesktopPlatformModule::Get();

	if (DesktopPlatform)
	{
		const FString DefaultPath = FEditorDirectories::Get().GetLastDirectory(ELastDirectory::GENERIC_EXPORT);

		TSharedPtr<SWindow> ParentWindow = FSlateApplication::Get().FindWidgetWindow(StringTableEntriesListView.ToSharedRef());
		const void* const ParentWindowHandle = (ParentWindow.IsValid() && ParentWindow->GetNativeWindow().IsValid())
			? ParentWindow->GetNativeWindow()->GetOSWindowHandle()
			: nullptr;

		TArray<FString> OutFiles;
		if (DesktopPlatform->SaveFileDialog(
			ParentWindowHandle,
			LOCTEXT("ExportStringTableTitle", "Choose a string table CSV file...").ToString(),
			DefaultPath,
			TEXT(""),
			TEXT("String Table CSV (*.csv)|*.csv"),
			EFileDialogFlags::None,
			OutFiles
			))
		{
			UStringTable* StringTable = Cast<UStringTable>(GetEditingObject());
			if (StringTable)
			{
				StringTable->GetStringTable()->ExportStringsToCSVFile(OutFiles[0]);
			}
		}
	}

	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE
