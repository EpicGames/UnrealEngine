// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Toolkits/IToolkitHost.h"
#include "IStringTableEditor.h"
#include "EditorUndoClient.h"

#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SMultiLineEditableTextBox.h"
#include "Widgets/Views/SListView.h"
#include "Misc/TextFilter.h"

/** Viewer/editor for a String Table */
class FStringTableEditor : public IStringTableEditor, public FEditorUndoClient
{
	friend class SStringTableEntryRow;

public:
	virtual void RegisterTabSpawners(const TSharedRef<class FTabManager>& TabManager) override;
	virtual void UnregisterTabSpawners(const TSharedRef<class FTabManager>& TabManager) override;

	/**
	 * Edits the specified string table
	 *
	 * @param	Mode					Asset editing mode for this editor (standalone or world-centric)
	 * @param	InitToolkitHost			When Mode is WorldCentric, this is the level editor instance to spawn this editor within
	 * @param	StringTable				The string table to edit
	 */
	void InitStringTableEditor(const EToolkitMode::Type Mode, const TSharedPtr<class IToolkitHost>& InitToolkitHost, UStringTable* StringTable);

	/** Constructor */
	FStringTableEditor() = default;

	/** Destructor */
	virtual ~FStringTableEditor();

	/** IToolkit interface */
	virtual FName GetToolkitFName() const override;
	virtual FText GetBaseToolkitName() const override;
	virtual FString GetWorldCentricTabPrefix() const override;
	virtual FLinearColor GetWorldCentricTabColorScale() const override;

	//~ IStringTableEditor
	virtual void RefreshStringTableEditor(const FString& NewSelection = FString()) override;

	//~ FEditorUndoClient
	virtual void PostUndo(bool bSuccess) override;
	virtual void PostRedo(bool bSuccess) override;
	void HandleUndoRedo();

	/** Get the string table being edited */
	const UStringTable* GetStringTable() const;

	/** Called after the string table has been changed */
	void HandlePostChange(const FString& NewSelection = FString());

private:
	/** Cached string table entry */
	struct FCachedStringTableEntry
	{
		FCachedStringTableEntry(FString InKey, FString InSourceString, FString InDevNotes)
			: Key(MoveTemp(InKey))
			, SourceString(MoveTemp(InSourceString))
			, DevNotes(MoveTemp(InDevNotes))
		{
		}

		FString Key;
		FString SourceString;
		FString DevNotes;
	};

	/**	Spawns the tab with the string table inside */
	TSharedRef<SDockTab> SpawnTab_StringTable(const FSpawnTabArgs& Args);

	/** Create the row for the given cached string table entry */
	TSharedRef<ITableRow> OnGenerateStringTableEntryRow(TSharedPtr<FCachedStringTableEntry> CachedStringTableEntry, const TSharedRef<STableViewBase>& Table);

	/** Refresh the cached string table editor UI */
	void RefreshCachedStringTable(const FString& InCachedSelection = FString());

	/** Get the current namespace used by this table */
	FText GetNamespace() const;

	/** Handle for the namespace being changed. Verify that the namespace is valid. */
	void OnNamespaceChanged(const FText& InText);

	/** Handler for the namespace being committed. */
	void OnNamespaceCommitted(const FText& InText, ETextCommit::Type);

	/** Set the value of an entry in the string table */
	void SetEntry(const FString& InKey, const FString& InSourceString, const FString& InDevNotes);

	/** Delete the given entry from the string table */
	void DeleteEntry(const FString& InKey);

	/** Check the given text to see whether it's valid to be used as identity (namespace or key) */
	bool IsValidIdentity(const FText& InIdentity, FText* OutReason = nullptr, const FText* InErrorCtx = nullptr) const;

	/** Handle for the key being changed. Verify that the key is valid and unique. */
	void OnKeyChanged(const FText& InText);

	/** Handler for the source string being committed. Treat as if "Add" were pressed. */
	void OnSourceStringCommitted(const FText& InText, ETextCommit::Type);

	/** Handler for the search box string being changed. */
	void OnFilterTextChanged(const FText& InNewText);

	/** Check function for context menu delete action. */
	bool CanExecuteDelete() const;

	/** Handler for context menu delete action. */
	void ExecuteDelete();

	/** Handler for when opening the context menu in the list view. */
	TSharedPtr<SWidget> OnConstructContextMenu();

	/** Handler for the "Add" button */
	FReply OnAddClicked();

	/** Handler for the "Import from CSV" button */
	FReply OnImportFromCSVClicked();

	/** Handler for the "Export to CSV" button */
	FReply OnExportToCSVClicked();

	/** Editable text for the namespace */
	TSharedPtr<SEditableTextBox> NamespaceEditableTextBox;

	/** Search box for the text entries */
	TSharedPtr<class SSearchBox> SearchBox;

	/** Text filter for text entries */
	using FEntryTextFilter = TTextFilter< TSharedPtr<FCachedStringTableEntry> >;
	TSharedPtr<FEntryTextFilter> EntryTextFilter;

	/** Editable text for the key */
	TSharedPtr<SEditableTextBox> KeyEditableTextBox;

	/** Editable text for the source string */
	TSharedPtr<SMultiLineEditableTextBox> SourceStringEditableTextBox;
	
	/** Editable text for the developer translation context notes */
	TSharedPtr<SMultiLineEditableTextBox> DevNotesEditableTextBox;

	/** Array of cached string table entries */
	TArray<TSharedPtr<FCachedStringTableEntry>> CachedStringTableEntries;

	/** List view showing the cached string table entries */
	TSharedPtr<SListView<TSharedPtr<FCachedStringTableEntry>>> StringTableEntriesListView;

	/** List of UI commands */
	TSharedPtr<class FUICommandList> CommandList;

	/**	The tab ID for the string table tab */
	static const FName StringTableTabId;

	/**	The column ID for the dummy column */
	static const FName StringTableDummyColumnId;

	/**	The column ID for the key column */
	static const FName StringTableKeyColumnId;

	/**	The column ID for the source string column */
	static const FName StringTableSourceStringColumnId;

	/** The column ID for the developer translation context notes column */
	static const FName StringTableDevNotesColumnId;

	/**	The column ID for the delete column */
	static const FName StringTableDeleteColumnId;
};
