// Copyright Epic Games, Inc. All Rights Reserved.

#include "StringTableEditorModule.h"

#include "AssetToolsModule.h"
#include "IAssetTools.h"
#include "Modules/ModuleManager.h"
#include "IStringTableEditor.h"
#include "Internationalization/StringTableRegistry.h"
#include "StringTableEditor.h"

#include "PackageMigrationContext.h"
#include "Internationalization/StringTable.h"

IMPLEMENT_MODULE(FStringTableEditorModule, StringTableEditor);

const FName FStringTableEditorModule::StringTableEditorAppIdentifier("StringTableEditorApp");

void FStringTableEditorModule::StartupModule()
{
	MenuExtensibilityManager = MakeShareable(new FExtensibilityManager);
	ToolBarExtensibilityManager = MakeShareable(new FExtensibilityManager);

	IAssetTools& AssetTools = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools").Get();
	AssetTools.GetOnPackageMigration().AddRaw(this, &FStringTableEditorModule::OnPackageMigration);
}

void FStringTableEditorModule::ShutdownModule()
{
	if (FAssetToolsModule* AssetToolsModule = FModuleManager::GetModulePtr<FAssetToolsModule>("AssetTools"))
	{
		AssetToolsModule->Get().GetOnPackageMigration().RemoveAll(this);
	}

	MenuExtensibilityManager.Reset();
	ToolBarExtensibilityManager.Reset();
}

TSharedRef<IStringTableEditor> FStringTableEditorModule::CreateStringTableEditor(const EToolkitMode::Type Mode, const TSharedPtr<IToolkitHost>& InitToolkitHost, UStringTable* StringTable)
{
	TSharedRef<FStringTableEditor> NewStringTableEditor(new FStringTableEditor());
	NewStringTableEditor->InitStringTableEditor(Mode, InitToolkitHost, StringTable);
	return NewStringTableEditor;
}

void FStringTableEditorModule::OnPackageMigration(UE::AssetTools::FPackageMigrationContext& MigrationContext)
{
	constexpr auto ForEachStringTableInTheWay = [](UE::AssetTools::FPackageMigrationContext& MigrationContext, TFunctionRef<void(UStringTable*)> Callback)
	{
		const TArray<UPackage*>& MovedPackages = MigrationContext.GetMovedOutOfTheWayPackages();
		for (UPackage* Package : MovedPackages)
		{
			UObject* MainAsset = Package->FindAssetInPackage();
			if (UStringTable* StringTableAsset = Cast<UStringTable>(MainAsset))
			{
				Callback(StringTableAsset);
			}
		}
	};

	// The migration can create temporally new packages that will use the same name as the asset migrated. So we need to unregister the migrated asset for the duration of the migration
	if (MigrationContext.GetCurrentStep() == UE::AssetTools::FPackageMigrationContext::EPackageMigrationStep::InTheWayPackagesMoved)
	{
		ForEachStringTableInTheWay(MigrationContext, [](UStringTable* StringTableAsset)
			{
				FStringTableRegistry::Get().UnregisterStringTable(StringTableAsset->GetStringTableId());
			});
	}
	else if (MigrationContext.GetCurrentStep() == UE::AssetTools::FPackageMigrationContext::EPackageMigrationStep::EndMigration)
	{
		ForEachStringTableInTheWay(MigrationContext, [](UStringTable* StringTableAsset)
			{
				FStringTableRegistry::Get().RegisterStringTable(StringTableAsset->GetStringTableId(), StringTableAsset->GetMutableStringTable());
			});
	}
}
