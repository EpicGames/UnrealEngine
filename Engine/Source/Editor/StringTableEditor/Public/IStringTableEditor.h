// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Toolkits/AssetEditorToolkit.h"

/** String Table Editor public interface */
class IStringTableEditor : public FAssetEditorToolkit
{
public:
	/**
	 * Notify the String Table editor that it needs to refresh due to an external String Table change.
	 */
	virtual void RefreshStringTableEditor(const FString& NewSelection = FString()) = 0;
};
