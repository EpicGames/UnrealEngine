// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Toolkits/IToolkitHost.h"
#include "Modules/ModuleInterface.h"
#include "Toolkits/AssetEditorToolkit.h"

#define UE_API STRINGTABLEEDITOR_API

class IStringTableEditor;
class UStringTable;

namespace UE::AssetTools
{
	struct FPackageMigrationContext;
}

/** String Table Editor module */
class FStringTableEditorModule : public IModuleInterface,
	public IHasMenuExtensibility, public IHasToolBarExtensibility
{
public:
	// IModuleInterface
	UE_API virtual void StartupModule() override;
	UE_API virtual void ShutdownModule() override;

	/**
	 * Creates an instance of string table editor object.
	 *
	 * @param	Mode					Mode that this editor should operate in
	 * @param	InitToolkitHost			When Mode is WorldCentric, this is the level editor instance to spawn this editor within
	 * @param	StringTable				The string table to start editing
	 *
	 * @return	Interface to the new string table editor
	 */
	UE_API TSharedRef<IStringTableEditor> CreateStringTableEditor(const EToolkitMode::Type Mode, const TSharedPtr<class IToolkitHost>& InitToolkitHost, UStringTable* StringTable);

	/** Gets the extensibility managers for outside entities to extend static mesh editor's menus and toolbars */
	virtual TSharedPtr<FExtensibilityManager> GetMenuExtensibilityManager() override { return MenuExtensibilityManager; }

	virtual TSharedPtr<FExtensibilityManager> GetToolBarExtensibilityManager() override { return ToolBarExtensibilityManager; }

	/** StringTable Editor app identifier string */
	static UE_API const FName StringTableEditorAppIdentifier;

private:
	UE_API void OnPackageMigration(UE::AssetTools::FPackageMigrationContext& MigrationContext);

	TSharedPtr<FExtensibilityManager> MenuExtensibilityManager;
	TSharedPtr<FExtensibilityManager> ToolBarExtensibilityManager;
};

#undef UE_API
