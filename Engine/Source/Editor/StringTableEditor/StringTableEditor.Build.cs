// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedTargetTypes(TargetType.Editor, TargetType.Program)]
public class StringTableEditor : ModuleRules
{
	public StringTableEditor(ReadOnlyTargetRules Target)
		 : base(Target)
	{
		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"AssetDefinition",
				"Core",
				"CoreUObject",
				"Engine",
				"InputCore",
				"Slate",
				"SlateCore",
				"DesktopPlatform",
				"EditorFramework",
				"UnrealEd",
				"AssetTools",
			});

		DynamicallyLoadedModuleNames.Add("WorkspaceMenuStructure");
	}
}
